{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "FastBuild Controller",
	"Description" : "Adds support for shader compiling distribution using FastBuild",
	"Category" : "Build Distribution",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,

	"Modules" :
	[
		{
			"Name" : "FastBuildController",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "EarliestPossible",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux"
			]
		}
	]
}
