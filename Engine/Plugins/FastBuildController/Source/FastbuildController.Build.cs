// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class FastBuildController : ModuleRules
{
	public FastBuildController(ReadOnlyTargetRules TargetRules)
		: base(TargetRules)
	{
		bRequiresPlatformSDK = true;
		
		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"RenderCore"
		});

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"DistributedBuildInterface",
				"TargetPlatform"
			}
		);
	}
}
