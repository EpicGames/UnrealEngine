// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastBuildControllerModule.h"
#include "Features/IModularFeatures.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "FastBuildJobProcessor.h"
#include "FastBuildUtilities.h"

DEFINE_LOG_CATEGORY_STATIC(LogFastBuildController, Log, Log);

namespace FASTBuildControllerVariables
{
	int32 Enabled = 1;
	FAutoConsoleVariableRef CVarFASTBuildControllerShaderCompile(
        TEXT("r.FASTBuildController.Enabled"),
        Enabled,
        TEXT("Enables or disables the use of FASTBuild to build shaders.\n")
        TEXT("0: Controller will not be used (shaders will be built locally or using other controllers). \n")
        TEXT("1: Distribute builds using FASTBuild."),
        ECVF_Default);
}

#if PLATFORM_MAC

#include "Apple/AppleStringUtils.h"
#include "Apple/ScopeAutoreleasePool.h"

static FString GetMetalCompilerFolder()
{
	FString Result;
	if (FPlatformProcess::ExecProcess(TEXT("/usr/bin/xcrun"), TEXT("--sdk macosx metal -v --target=air64-apple-darwin18.7.0"), nullptr, &Result, &Result))
	{
		const TCHAR InstalledDirText[] = TEXT("InstalledDir:");
		int32 InstalledDirOffset = Result.Find(InstalledDirText, ESearchCase::CaseSensitive);
		if (InstalledDirOffset != INDEX_NONE)
		{
			InstalledDirOffset += UE_ARRAY_COUNT(InstalledDirText);
			const int32 MacOSBinOffset = Result.Find(TEXT("/macos/bin\n"), ESearchCase::CaseSensitive, ESearchDir::FromStart, InstalledDirOffset);
			if (MacOSBinOffset != INDEX_NONE)
			{
				FString Substring = Result.Mid(InstalledDirOffset, MacOSBinOffset - InstalledDirOffset);
				return Result.Mid(InstalledDirOffset, MacOSBinOffset - InstalledDirOffset);
			}
		}
	}

	return FString();
}
#endif // PLATFORM_MAC

FFastBuildControllerModule::FFastBuildControllerModule()
	: bSupported(false)
	, bModuleInitialized(false)
	, bControllerInitialized(false)
	, TasksCS(new FCriticalSection())
	, RootWorkingDirectory(FString::Printf(TEXT("%sUnrealFastBuildWorkingDir/"), FPlatformProcess::UserTempDir()))
	, WorkingDirectory(RootWorkingDirectory + FGuid::NewGuid().ToString(EGuidFormats::Digits))
	, bShutdown(false)
{
	const FGuid Guid = FGuid::NewGuid();
	IntermediateShadersDirectory = FPaths::EngineIntermediateDir() / TEXT("Shaders") / TEXT("FB") / FApp::GetProjectName();
}

FFastBuildControllerModule::~FFastBuildControllerModule()
{
	if (JobDispatcherThread)
	{
		JobDispatcherThread->Stop();
	}
	
	CleanWorkingDirectory();
}

bool FFastBuildControllerModule::IsSupported()
{
	if (bControllerInitialized)
	{
		return bSupported;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("nofastbuildcontroller")) ||
		FParse::Param(FCommandLine::Get(), TEXT("nofastbuildshadercompile")) ||
		FParse::Param(FCommandLine::Get(), TEXT("nofbuildshadercompile")) ||
		FParse::Param(FCommandLine::Get(), TEXT("noshaderworker")))
	{
		FASTBuildControllerVariables::Enabled = 0;
	}

	// Check to see if the FASTBuild exe exists.
	if (FASTBuildControllerVariables::Enabled == 1)
	{
		const FString BrokeragePath = FPlatformMisc::GetEnvironmentVariable(TEXT("FASTBUILD_BROKERAGE_PATH"));
		if (BrokeragePath.IsEmpty())
		{
			FASTBuildControllerVariables::Enabled = 0;
			return false;
		}

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		const FString FASTBuildAbsolutePath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*FastBuildUtilities::GetFastBuildExecutablePath());
		if (!PlatformFile.FileExists(*FASTBuildAbsolutePath))
		{
			UE_LOGF(LogFastBuildController, Warning, "Cannot use FASTBuild Shader Compiler as FASTBuild is not found: %ls",
				*FPaths::ConvertRelativePathToFull(FASTBuildAbsolutePath));
			FASTBuildControllerVariables::Enabled = 0;
			return false;
		}	

#if PLATFORM_MAC
		static bool bCopyMetalCompilerToIntermediateDir = true;
		if (bCopyMetalCompilerToIntermediateDir)
		{
			SCOPED_AUTORELEASE_POOL;

			// Make a copy of all the Metal shader compiler files in the intermediate folder, so that they are in the same directory tree as SharedCompileWorker.
			// This is required for FASTBuild to preserve the directory structure when it copies these files to the worker
			const FString SrcDir = GetMetalCompilerFolder();
			if (SrcDir.Len() == 0)
			{
				UE_LOGF(LogFastBuildController, Warning, "Cannot use FASTBuild Shader Compiler as Metal shader compiler could not be found");
				FASTBuildControllerVariables::Enabled = 0;
			}
			else
			{
				const FString IntermediateShadersDir = FPaths::EngineIntermediateDir() / TEXT("Shaders");
				const FString DestDir = IntermediateShadersDir / TEXT("metal");
				if (PlatformFile.DirectoryExists(*DestDir))
				{
					PlatformFile.DeleteDirectoryRecursively(*DestDir);
				}

				if (!PlatformFile.DirectoryExists(*IntermediateShadersDir))
				{
					PlatformFile.CreateDirectoryTree(*IntermediateShadersDir);
				}

				NSFileManager* FileManager = [NSFileManager defaultManager]; // Use NSFileManager as PlatformFile's CopyDirectoryTree does not preserve file modification times
				const bool bCopied = [FileManager copyItemAtPath:SrcDir.GetNSString() toPath:DestDir.GetNSString() error:nil];
				if (!bCopied)
				{
					UE_LOGF(LogFastBuildController, Warning, "Cannot use FASTBuild Shader Compiler as Metal shader compiler could not be copied to the intermediate folder: %ls -> %ls",
						*SrcDir, *DestDir);
					FASTBuildControllerVariables::Enabled = 0;
				}
			}

			bCopyMetalCompilerToIntermediateDir = false;
		}

		if (FASTBuildControllerVariables::Enabled == 1)
		{
			UE_LOGF(LogFastBuildController, Warning, "FASTBuild Shader Compiler is temporarily disabled on Mac until problems with locating Autogen shader folder are sorted out");
			FASTBuildControllerVariables::Enabled = 0;
		}
#endif // PLATFORM_MAC
	}

	bSupported = FASTBuildControllerVariables::Enabled == 1;

	return bSupported;
}

FString FFastBuildControllerModule::CreateUniqueFilePath()
{
	check(bSupported);
	return FString::Printf(TEXT("%s/%d-fbuild"), *WorkingDirectory, NextFileID.Increment()); 
}

void FFastBuildControllerModule::StartupModule()
{
	check(!bModuleInitialized);

	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureType(), this);
	
	bModuleInitialized = true;
}

void FFastBuildControllerModule::ShutdownModule()
{
	check(bModuleInitialized);

	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureType(), this);

	// Stop the jobs thread
	if (JobDispatcherThread)
	{
		JobDispatcherThread->Stop();	
	}

	// Cancel any remaining tasks
	for (auto Iter = DispatchedTasks.CreateIterator(); Iter; ++Iter)
	{
		FDistributedBuildTask* Task = Iter.Value();
		Task->Cancel();
		delete Task;
	}

	FDistributedBuildTask* Task;
	while (PendingTasks.Dequeue(Task))
	{
		Task->Cancel();
		delete Task;
	}

	CleanWorkingDirectory();
	bModuleInitialized = false;
	bControllerInitialized = false;
}

void FFastBuildControllerModule::InitializeController()
{
	if (!IFileManager::Get().DirectoryExists(*RootWorkingDirectory))
	{
		IFileManager::Get().MakeDirectory(*RootWorkingDirectory);
	}

	IFileManager::Get().DeleteDirectory(*IntermediateShadersDirectory, false, true);

	if (IsSupported())
	{
		JobDispatcherThread = MakeUnique<FFastBuildJobProcessor>(*this);
		JobDispatcherThread->StartThread();
	}

	bControllerInitialized = true;
}

FString FFastBuildControllerModule::RemapPath(const FString& SourcePath) const
{
	if (!FPaths::IsUnderDirectory(SourcePath, FPaths::RootDir()))
	{
		FString DestinationPath = SourcePath;
		FString AbsoluteProjectDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
		if (DestinationPath.StartsWith(AbsoluteProjectDir))
		{
			DestinationPath = DestinationPath.Mid(AbsoluteProjectDir.Len());
		}
		if (DestinationPath.StartsWith(TEXT("\\\\"), ESearchCase::CaseSensitive))
		{
			DestinationPath = TEXT("____") + DestinationPath.Mid(2);
		}
		DestinationPath.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);
		if (DestinationPath.Len() > 1 && DestinationPath[1] == ':')
		{
			DestinationPath = DestinationPath.Left(1) + TEXT("__") / DestinationPath.Mid(2);
		}
		if (DestinationPath.Len() > 0 && DestinationPath[0] == '/')
		{
			DestinationPath = TEXT("__") + DestinationPath.Mid(1);
		}
		DestinationPath = IntermediateShadersDirectory / DestinationPath;

		return DestinationPath;
	}
	else
	{
		return SourcePath;
	}
}

void FFastBuildControllerModule::CleanWorkingDirectory()
{
	IFileManager& FileManager = IFileManager::Get();
	
	if (!WorkingDirectory.IsEmpty())
	{
		if (!FileManager.DeleteDirectory(*WorkingDirectory, false, true))
		{
			UE_LOGF(LogFastBuildController, Log, "%ls => Failed to delete current working Directory => %ls", ANSI_TO_TCHAR(__FUNCTION__), *RootWorkingDirectory);
		}
	}

	if (!IntermediateShadersDirectory.IsEmpty())
	{
		if (!FileManager.DeleteDirectory(*IntermediateShadersDirectory, false, true))
		{
			UE_LOGF(LogFastBuildController, Log, "%ls => Failed to delete directory => %ls", ANSI_TO_TCHAR(__FUNCTION__), *IntermediateShadersDirectory);
		}	
	}
}

TFuture<FDistributedBuildTaskResult> FFastBuildControllerModule::EnqueueTask(const FTaskCommandData& CommandData)
{
	check(bSupported);

	TPromise<FDistributedBuildTaskResult> Promise;
	TFuture<FDistributedBuildTaskResult> Future = Promise.GetFuture();

	// Enqueue the new task
	FDistributedBuildTask* Task = new FDistributedBuildTask(NextTaskID.Increment(), CommandData, MoveTemp(Promise));
	{
		FScopeLock Lock(TasksCS.Get());
		PendingTasks.Enqueue(Task);
		PendingTasksCounter.Increment();
	}

	return MoveTemp(Future);
}

void FFastBuildControllerModule::EnqueueTask(FDistributedBuildTask* Task)
{
	FScopeLock Lock(TasksCS.Get());
	{
		PendingTasks.Enqueue(Task);
	}
	PendingTasksCounter.Increment();
}

FDistributedBuildTask* FFastBuildControllerModule::DequeueTask()
{
	FDistributedBuildTask* TaskPtr;
	
	FScopeLock Lock(TasksCS.Get());
	{
		PendingTasks.Dequeue(TaskPtr);
	}
	
	PendingTasksCounter.Increment();
	return TaskPtr;
}

void FFastBuildControllerModule::RegisterDispatchedTask(FDistributedBuildTask* DispatchedTask)
{
	check(DispatchedTask)
	{
		FScopeLock Lock(TasksCS.Get());
		DispatchedTasks.Add(DispatchedTask->ID , DispatchedTask);
	}
}

void FFastBuildControllerModule::ReEnqueueDispatchedTasks()
{
	FScopeLock Lock(TasksCS.Get());
	// Reclaim dispatched (incomplete) tasks 
	for (const TPair<uint32, FDistributedBuildTask*>& DispatchedTaskEntry : DispatchedTasks)
	{
		EnqueueTask(DispatchedTaskEntry.Value);
	}
			
	DispatchedTasks.Reset();
}

void FFastBuildControllerModule::DeRegisterDispatchedTasks(const TArray<uint32>& InTasksID)
{
	for (const uint32& TaskID : InTasksID)
	{
		FScopeLock Lock(TasksCS.Get());
		DispatchedTasks.Remove(TaskID);
	}
}

void FFastBuildControllerModule::ReportJobProcessed(FDistributedBuildTask* CompletedTask, const FTaskResponse& InTaskResponse)
{
	CompletedTask->Finalize(InTaskResponse.ReturnCode);
	delete CompletedTask;
}

FFastBuildControllerModule& FFastBuildControllerModule::Get()
{
	return FModuleManager::LoadModuleChecked<FFastBuildControllerModule>(TEXT("FastBuildController"));
}

IMPLEMENT_MODULE(FFastBuildControllerModule, FastBuildController);
