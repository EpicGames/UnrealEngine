// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastBuildJobProcessor.h"
#include "FastBuildControllerModule.h"
#include "FastBuildUtilities.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformTime.h"
#include "Misc/Paths.h"
#include "HAL/RunnableThread.h"

namespace FastBuildJobProcessorOptions
{
	float SleepTimeBetweenActions = 0.1f;
	FAutoConsoleVariableRef CVarFASTBuildSleepTimeBetweenActions(
        TEXT("r.FASTBuild.JobProcessor.SleepTimeBetweenActions"),
        SleepTimeBetweenActions,
        TEXT("How much time the job processor thread should sleep between actions .\n"));


	int32 MinJobsToKickOff = 100;
	FAutoConsoleVariableRef CVarFASTBuildShaderMinJobsToKickOff(
        TEXT("r.FASTBuild.JobProcessor.MinBatchSize"),
        MinJobsToKickOff,
        TEXT("Minimum number of shaders to compile with FASTBuild.\n")
        TEXT("Default = 100\n"),
        ECVF_Default);

	int32 MaxTimeWithPendingJobs = 10;
	FAutoConsoleVariableRef CVarFASTBuildShaderMaxTimeWithPendingJobs(
        TEXT("r.FASTBuild.JobProcessor.MaxTimeWithPendingJobs"),
        MaxTimeWithPendingJobs,
        TEXT("Specifies how much time in seconds we will wait to have the min amount of pending jobs. Past this time, the build will start anyways.\n")
        TEXT("Default = 10\n"),
        ECVF_Default);
}


DEFINE_LOG_CATEGORY_STATIC(LogFastBuildJobProcessor, Log, Log);

static const FString FASTBuildScriptFileName(TEXT("shaders.bff"));

FFastBuildJobProcessor::FFastBuildJobProcessor(FFastBuildControllerModule& InControllerModule)
	: Thread(nullptr),
	  BuildProcessID(0),
	  LastTimeKickedOffJobs(0),
	  ControllerModule(InControllerModule),
	  bIsWorkDone(false)
{
}

FFastBuildJobProcessor::~FFastBuildJobProcessor()
{
	if (BuildProcessHandle.IsValid())
	{
		// We still have a build in progress, so we need to terminate it.
		FPlatformProcess::TerminateProc(BuildProcessHandle);
		FPlatformProcess::CloseProc(BuildProcessHandle);
		FPlatformProcess::ClosePipe(PipeRead, PipeWrite);
	}

	if (Thread)
	{
		delete Thread;
		Thread = nullptr;
	}
}

uint32 FFastBuildJobProcessor::Run()
{
	bIsWorkDone = false;
	LastTimeKickedOffJobs  = FPlatformTime::Cycles();
	
	while (!bForceStop)
	{
		const float ElapsedSeconds = (FPlatformTime::Cycles() - LastTimeKickedOffJobs) * FPlatformTime::GetSecondsPerCycle();
		const bool bShouldKickoffJobsAnyway = ElapsedSeconds > FastBuildJobProcessorOptions::MaxTimeWithPendingJobs && ControllerModule.GetPendingTasksAmount() > 0;

		
		if (bShouldKickoffJobsAnyway || ControllerModule.GetPendingTasksAmount() > FastBuildJobProcessorOptions::MinJobsToKickOff)
		{
			LastTimeKickedOffJobs = FPlatformTime::Cycles();
			SubmitPendingJobs();
		}
	
		if (ControllerModule.AreTasksDispatched())
		{
			MonitorFastBuildProcess();
			GatherBuildResults();
		}	

		FPlatformProcess::Sleep(FastBuildJobProcessorOptions::SleepTimeBetweenActions);
	}

	bIsWorkDone = true;
	return 0;
}

void FFastBuildJobProcessor::StartThread()
{
	Thread = FRunnableThread::Create(this, TEXT("FastBuildJobProcessor"), 0, TPri_Normal, FPlatformAffinity::GetPoolThreadMask());
}

void FFastBuildJobProcessor::MonitorFastBuildProcess()
{
	if (!BuildProcessHandle.IsValid())
	{
		return;
	}
	
	bool bDoExitCheck = false;
	if (FPlatformProcess::IsProcRunning(BuildProcessHandle))
	{
		const FString STDOutput = FPlatformProcess::ReadPipe(PipeRead);
		if (STDOutput.Len() > 0)
		{
			TArray<FString> Lines;
			STDOutput.ParseIntoArrayLines(Lines);
			for (const FString& Line : Lines)
			{
				UE_LOGF(LogFastBuildJobProcessor, Verbose, "%ls", *Line);
			}
		}

		if (!ControllerModule.AreTasksDispatchedOrPending())
		{
			// We've processed all batches.
			// Wait for the FASTBuild console process to exit
			FPlatformProcess::WaitForProc(BuildProcessHandle);
			bDoExitCheck = true;
		}
	}
	else
	{
		bDoExitCheck = true;
	}
	
	if (bDoExitCheck)
	{	
		// The build process has stopped.
		// Do one final pass over the output files to gather any remaining results.
		GatherBuildResults();

		// The build process is no longer running.
		// We need to check the return code for possible failure
		int32 ReturnCode = 0;
		FPlatformProcess::GetProcReturnCode(BuildProcessHandle, &ReturnCode);

		switch (ReturnCode)
		{
			case FBUILD_OK:
				// No error
				break;

			case FBUILD_ERROR_LOADING_BFF:
				UE_LOGF(LogFastBuildJobProcessor, Fatal, "There was an error parsing FASTBuild script. This might be due to platform dependencies having files with the same names (Code %d).", ReturnCode);
				break;
			case FBUILD_BUILD_FAILED:
	        case FBUILD_BAD_ARGS:
	        case FBUILD_FAILED_TO_SPAWN_WRAPPER:
	        case FBUILD_FAILED_TO_SPAWN_WRAPPER_FINAL:
	        case FBUILD_WRAPPER_CRASHED:
	            // One or more of the shader compile worker processes crashed.
	            UE_LOGF(LogFastBuildJobProcessor, Fatal, "An error occurred during an FASTBuild shader compilation job. One or more of the shader compile worker processes exited unexpectedly (Code %d).", ReturnCode);
				break;
			default:
				UE_LOGF(LogFastBuildJobProcessor, Display, "An unknown error occurred during an FASTBuild shader compilation job (Code %d). Incomplete shader jobs will be redispatched in another FASTBuild build.", ReturnCode);
				break;
			case FBUILD_ALREADY_RUNNING:
				UE_LOGF(LogFastBuildJobProcessor, Display, "FASTBuild is already running. Incomplete shader jobs will be redispatched in another FASTBuild build.");
				break;
		}

		if (ReturnCode != FBUILD_OK && ReturnCode != FBUILD_ALREADY_RUNNING)
		{
			ControllerModule.ReEnqueueDispatchedTasks();
		}

		BuildProcessHandle.Reset();
	}
}

void FFastBuildJobProcessor::WriteScriptFileToDisk(const TArray<FDistributedBuildTask*>& PendingTasks, const FString& ScriptFilename, const FString& WorkerName) const
{
	// Create the FASTBuild script file.
	TUniquePtr<FArchive> ScriptFile(FastBuildUtilities::CreateFileHelper(ScriptFilename));
	check(ScriptFile.IsValid());
	{
		FastBuildUtilities::FASTBuildWriteScriptFileHeader(PendingTasks, *ScriptFile, WorkerName);
		
		// Write the task line for each shader batch
		for (const FDistributedBuildTask* CompilationTask : PendingTasks)
		{
			FString WorkerAbsoluteDirectory = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*ControllerModule.GetWorkingDirectory());
			FPaths::NormalizeDirectoryName(WorkerAbsoluteDirectory);

			const FString ExecFunction = FString::Printf(
                TEXT("ObjectList('ShaderBatch-%d')" LINE_TERMINATOR_ANSI)
                TEXT("{" LINE_TERMINATOR_ANSI)
                TEXT("\t.Compiler = 'ShaderCompiler'" LINE_TERMINATOR_ANSI)
                TEXT("\t.CompilerOptions = '\"\" %d %d \"%%1\" \"%%2\"'" LINE_TERMINATOR_ANSI)
                TEXT("\t.CompilerOutputExtension = '.out'"  LINE_TERMINATOR_ANSI)
                TEXT("\t.CompilerInputFiles = { '%s' }" LINE_TERMINATOR_ANSI)
                TEXT("\t.CompilerOutputPath = '%s'" LINE_TERMINATOR_ANSI)
                TEXT("}"  LINE_TERMINATOR_ANSI LINE_TERMINATOR_ANSI),
                CompilationTask->ID,
                FGenericPlatformProcess::GetCurrentProcessId(),
                CompilationTask->ID,
                *CompilationTask->CommandData.InputFileName,
                *WorkerAbsoluteDirectory);

			ScriptFile->Serialize((void*)StringCast<ANSICHAR>(*ExecFunction, ExecFunction.Len()).Get(), sizeof(ANSICHAR) * ExecFunction.Len());
		}

		const FString AliasBuildTargetOpen = FString(
            TEXT("Alias('all')" LINE_TERMINATOR_ANSI)
            TEXT("{" LINE_TERMINATOR_ANSI)
            TEXT("\t.Targets = { " LINE_TERMINATOR_ANSI)
        );
		ScriptFile->Serialize((void*)StringCast<ANSICHAR>(*AliasBuildTargetOpen, AliasBuildTargetOpen.Len()).Get(), sizeof(ANSICHAR) * AliasBuildTargetOpen.Len());

		for (FDistributedBuildTask* CompilationTask : PendingTasks)
		{
			const FString TargetExport = FString::Printf(TEXT("'ShaderBatch-%d', "), CompilationTask->ID);
			ScriptFile->Serialize((void*)StringCast<ANSICHAR>(*TargetExport, TargetExport.Len()).Get(), sizeof(ANSICHAR) * TargetExport.Len());

			ControllerModule.RegisterDispatchedTask(CompilationTask);
		}
	}

	const FString AliasBuildTargetClose = FString(TEXT(" }"  LINE_TERMINATOR_ANSI "}"  LINE_TERMINATOR_ANSI));
	ScriptFile->Serialize((void*)StringCast<ANSICHAR>(*AliasBuildTargetClose, AliasBuildTargetClose.Len()).Get(), sizeof(ANSICHAR) * AliasBuildTargetClose.Len());
	ScriptFile = nullptr;
}

void FFastBuildJobProcessor::SubmitPendingJobs()
{
	if (BuildProcessHandle.IsValid() && FPlatformProcess::IsProcRunning(BuildProcessHandle))
	{
		return;
	}

	if (!ControllerModule.AreTasksPending())
	{
		return;
	}

	TArray<FDistributedBuildTask*> TasksToSubmit;
	FDistributedBuildTask* PendingTask = nullptr;
	{
		FScopeLock Lock(ControllerModule.GetTasksCS());
		while (ControllerModule.AreTasksPending())
		{
			PendingTask = ControllerModule.DequeueTask();
			TasksToSubmit.Add(PendingTask);
		}
	}
	
	const FString ScriptFilename = ControllerModule.GetWorkingDirectory() / FASTBuildScriptFileName;
	const FString WorkerName = TasksToSubmit[0]->CommandData.Command;
	const uint32 DispatcherID = TasksToSubmit[0]->CommandData.DispatcherPID;

	// Create the Fast build script file with all the current pending jobs
	WriteScriptFileToDisk(TasksToSubmit, ScriptFilename, WorkerName);

	const FString FASTBuildConsoleArgs = TEXT("-config \"") + ScriptFilename + TEXT("\" -dist -clean -monitor");
	
	// Kick off the FASTBuild process...
	verify(FPlatformProcess::CreatePipe(PipeRead, PipeWrite));
	BuildProcessHandle = FPlatformProcess::CreateProc(*FastBuildUtilities::GetFastBuildExecutablePath(), *FASTBuildConsoleArgs, false, false, true, &BuildProcessID, 0, nullptr, PipeWrite);
	if (!BuildProcessHandle.IsValid())
	{
		UE_LOGF(LogFastBuildJobProcessor, Fatal, "Failed to launch %ls during shader compilation.", *FastBuildUtilities::GetFastBuildExecutablePath());
	}

	// If the engine crashes, we don't get a chance to kill the build process.
	// Start up the build monitor process to monitor for engine crashes.
	if (WorkerName.Contains(TEXT("ShaderCompile")))
	{
		uint32 BuildMonitorProcessID;
		FProcHandle BuildMonitorHandle = FPlatformProcess::CreateProc(*WorkerName, *FString::Printf(TEXT("-xgemonitor %d %d"), DispatcherID, BuildProcessID), true, false, false, &BuildMonitorProcessID, 0, nullptr, nullptr);
		FPlatformProcess::CloseProc(BuildMonitorHandle);
	}
}

void FFastBuildJobProcessor::GatherBuildResults() const
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	IFileManager& FileManager = IFileManager::Get();
	
	TArray<uint32> CompletedTasks;
	for (const TPair<uint32, FDistributedBuildTask*>& DispatchedTaskEntry : ControllerModule.GetDispatchedTasks())
	{
		FDistributedBuildTask* CompileTask = DispatchedTaskEntry.Value;

		constexpr uint64 VersionAndFileSizeSize = sizeof(uint32) + sizeof(uint64);	
		if (PlatformFile.FileExists(*CompileTask->CommandData.OutputFileName) &&
			FileManager.FileSize(*CompileTask->CommandData.OutputFileName) > VersionAndFileSizeSize)
		{
			TUniquePtr<FArchive> OutputFilePtr(FileManager.CreateFileReader(*CompileTask->CommandData.OutputFileName, FILEREAD_Silent));
			if (OutputFilePtr)
			{
				FArchive& OutputFile = *OutputFilePtr;
				int32 OutputVersion;
				OutputFile << OutputVersion; // NOTE (SB): Do not care right now about the version.
				int64 FileSize = 0;
				OutputFile << FileSize;

				// NOTE (SB): Check if we received the full file yet.
				if (OutputFile.TotalSize() >= FileSize)
				{
					FTaskResponse TaskCompleted;
					TaskCompleted.ID = CompileTask->ID;
					TaskCompleted.ReturnCode = 0;
					
					ControllerModule.ReportJobProcessed(CompileTask, TaskCompleted);
					CompletedTasks.Add(DispatchedTaskEntry.Key);
				}
			}
		}
	}
	ControllerModule.DeRegisterDispatchedTasks(CompletedTasks);
}
