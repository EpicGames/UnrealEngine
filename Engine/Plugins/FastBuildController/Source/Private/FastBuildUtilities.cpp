// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastBuildUtilities.h"
#include "FastBuildControllerModule.h"

#include "ShaderCore.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/CoreMisc.h"
#include "Misc/Paths.h"

namespace FASTBuildControllerUtilitiesVariables
{
	int32 SendWorkerApplicationDebugSymbols = 0;
	FAutoConsoleVariableRef CVarFASTBuildSendDebugSymbols(
        TEXT("r.FASTBuildController.SendSCWDebugSymbols"),
		SendWorkerApplicationDebugSymbols,
        TEXT("Enable when distributed shader compiler workers crash.\n")
        TEXT("0: Do not send along debug information in FASTBuild. \n")
        TEXT("1: Send along debug information in FASTBuild."),
        ECVF_Default);
}

void FastBuildUtilities::WriteShaderWorkerDependenciesToFile(FArchive& ScriptFile)
{
	FDependencyEnumerator DllDeps = FDependencyEnumerator(ScriptFile, TEXT("ShaderCompileWorker-"), PLATFORM_WINDOWS == 1 ? TEXT(".dll") : (PLATFORM_MAC == 1 ? TEXT(".dylib") : TEXT(".so")));
	IFileManager::Get().IterateDirectoryRecursively(*FPlatformProcess::GetModulesDirectory(), DllDeps);
	FDependencyEnumerator ModulesDeps = FDependencyEnumerator(ScriptFile, TEXT("ShaderCompileWorker"), TEXT(".modules"));
	IFileManager::Get().IterateDirectoryRecursively(*FPlatformProcess::GetModulesDirectory(), ModulesDeps);
	if (FASTBuildControllerUtilitiesVariables::SendWorkerApplicationDebugSymbols)
	{
#if PLATFORM_WINDOWS
		FDependencyEnumerator PdbDeps = FDependencyEnumerator(ScriptFile, TEXT("ShaderCompileWorker"), TEXT(".pdb"));
		IFileManager::Get().IterateDirectoryRecursively(*FPlatformProcess::GetModulesDirectory(), PdbDeps);
#endif
	}

	FDependencyEnumerator IniDeps = FDependencyEnumerator(ScriptFile, nullptr, TEXT(".ini"));
	TArray<FString> EngineConfigDirs = FPaths::GetExtensionDirs(FPaths::EngineDir(), TEXT("Config"));
	for (const FString& ConfigDir : EngineConfigDirs)
	{
		IFileManager::Get().IterateDirectoryRecursively(*ConfigDir, IniDeps);
	}
}

FString FastBuildUtilities::ReplaceEnvironmentVariablesInPath(const FString& ExtraFilePartialPath)
{
	FString ParsedPath;

	// Fast build cannot read environmental variables easily
	// Is better to resolve them here
	if (ExtraFilePartialPath.Contains(TEXT("%")))
	{
		TArray<FString> PathSections;
		ExtraFilePartialPath.ParseIntoArray(PathSections,TEXT("/"));

		for (FString& Section : PathSections)
		{
			if (Section.Contains(TEXT("%")))
			{
				Section.RemoveFromStart(TEXT("%"));
				Section.RemoveFromEnd(TEXT("%"));
				Section = FPlatformMisc::GetEnvironmentVariable(*Section);
			}
		}

		for (FString& Section : PathSections)
		{
			ParsedPath /= Section;
		}
			
		FPaths::NormalizeDirectoryName(ParsedPath);
	}

	if (ParsedPath.IsEmpty())
	{
		ParsedPath = ExtraFilePartialPath;
	}
	
	return ParsedPath;
}

void FastBuildUtilities::WriteDependenciesToFileUsingWildcardPath(FArchive& ScriptFile, FString ParsedPath)
{
	if (ParsedPath.Contains(TEXT("*")))
	{
		TArray<FString> PathSections;
		ParsedPath.ParseIntoArray(PathSections,TEXT("/"));

		FString PartialPath;
		for (const FString& Section : PathSections)
		{
			if (Section.Contains(TEXT("*")))
			{
				FString Prefix;
				FString Extension;
				Section.Split(TEXT("*"),&Prefix,&Extension);
		    			
				FDependencyEnumerator WildcardQuery = FDependencyEnumerator(ScriptFile, *Prefix, *Extension);
				IFileManager::Get().IterateDirectoryRecursively(*PartialPath, WildcardQuery);
				break;
			}
			PartialPath /= Section;
		}
	}
}

void FastBuildUtilities::WritePlatformCompilerDependenciesToFile(FArchive& ScriptFile)
{
	ITargetPlatformManagerModule* TargetPlatformManager = GetTargetPlatformManager();
	TArray<FString> FASTBuild_Toolchain;
	for (ITargetPlatform* TargetPlatform : TargetPlatformManager->GetTargetPlatforms())
	{
		TargetPlatform->GetShaderCompilerDependencies(FASTBuild_Toolchain);
	}

	for (const FString& ExtraFilePartialPath : FASTBuild_Toolchain)
	{
		FString ParsedPath = ReplaceEnvironmentVariablesInPath(ExtraFilePartialPath);

		if (ParsedPath.Contains(TEXT("*")))
		{
			// Fast Build cannot resolve wildcards
		    // We need to resolve them here
		    WriteDependenciesToFileUsingWildcardPath(ScriptFile, ParsedPath);
		}
		else
		{
			ParsedPath = TEXT("\t\t'") + IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*(ParsedPath)) + TEXT("'," LINE_TERMINATOR_ANSI);
			ScriptFile.Serialize((void*)StringCast<ANSICHAR>(*ParsedPath, ParsedPath.Len()).Get(), sizeof(ANSICHAR) * ParsedPath.Len());
		}
	
	}
}

void FastBuildUtilities::FASTBuildWriteScriptFileHeader(const TArray<FDistributedBuildTask*>& InCompilationTasks, FArchive& ScriptFile, const FString& WorkerName)
{	
	static constexpr TCHAR HeaderTemplate[] =
		TEXT("Settings" LINE_TERMINATOR_ANSI)
		TEXT("{" LINE_TERMINATOR_ANSI)
		TEXT("\t.CachePath = '%s'" LINE_TERMINATOR_ANSI)
		TEXT("}" LINE_TERMINATOR_ANSI)
		TEXT(LINE_TERMINATOR_ANSI)
		TEXT("Compiler('ShaderCompiler')" LINE_TERMINATOR_ANSI)
		TEXT("{" LINE_TERMINATOR_ANSI)
		TEXT("\t.CompilerFamily = 'custom'" LINE_TERMINATOR_ANSI)
		TEXT("\t.Executable = '%s'" LINE_TERMINATOR_ANSI)
		TEXT("\t.ExecutableRootPath = '%s'" LINE_TERMINATOR_ANSI)
		TEXT("\t.SimpleDistributionMode = true" LINE_TERMINATOR_ANSI)
		TEXT("\t.ExtraFiles = " LINE_TERMINATOR_ANSI)
		TEXT("\t{" LINE_TERMINATOR_ANSI);

	const FString HeaderString = FString::Printf(HeaderTemplate, *FastBuildUtilities::GetFastBuildCache(), *WorkerName,
			*IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FPaths::RootDir()));
	ScriptFile.Serialize((void*)StringCast<ANSICHAR>(*HeaderString, HeaderString.Len()).Get(),
			sizeof(ANSICHAR) * HeaderString.Len());

	WritePlatformCompilerDependenciesToFile(ScriptFile);

	WriteShaderWorkerDependenciesToFile(ScriptFile);

	const FString ExtraFilesFooter =
		TEXT("\t}" LINE_TERMINATOR_ANSI)
		TEXT("}" LINE_TERMINATOR_ANSI);
	ScriptFile.Serialize((void*)StringCast<ANSICHAR>(*ExtraFilesFooter, ExtraFilesFooter.Len()).Get(), sizeof(ANSICHAR) * ExtraFilesFooter.Len());
}

FArchive* FastBuildUtilities::CreateFileHelper(const FString& InFileName)
{
	// TODO: This logic came from FShaderCompileThreadRunnable::WriteNewTasks().
	// We can't avoid code duplication unless we refactored the local worker too.

	FArchive* File = nullptr;
	int32 RetryCount = 0;
	// Retry over the next two seconds if we can't write out the file.
	// Anti-virus and indexing applications can interfere and cause this to fail.
	while (File == nullptr && RetryCount < 200)
	{
		if (RetryCount > 0)
		{
			FPlatformProcess::Sleep(0.01f);
		}
		File = IFileManager::Get().CreateFileWriter(*InFileName, FILEWRITE_EvenIfReadOnly);
		RetryCount++;
	}
	if (File == nullptr)
	{
		File = IFileManager::Get().CreateFileWriter(*InFileName, FILEWRITE_EvenIfReadOnly | FILEWRITE_NoFail);
	}
	checkf(File, TEXT("Failed to create file %s!"), *InFileName);
	return File;
}

FString FastBuildUtilities::GetFastBuildExecutablePath()
{
	FString FASTBuildExecutablePath = TEXT("PLATFORM NOT SUPPORTED");
#if PLATFORM_WINDOWS
	FASTBuildExecutablePath = FPaths::EngineDir() / TEXT("Extras\\ThirdPartyNotUE\\FASTBuild\\Win64\\FBuild.exe");
#elif PLATFORM_MAC
	FASTBuildExecutablePath = FPaths::EngineDir() / TEXT("Extras/ThirdPartyNotUE/FASTBuild/Mac/FBuild");
#elif PLATFORM_LINUX
	FASTBuildExecutablePath = FPaths::EngineDir() / TEXT("Extras/ThirdPartyNotUE/FASTBuild/Linux/fbuild");
#endif

	return FASTBuildExecutablePath;
}

FString FastBuildUtilities::GetFastBuildCache()
{
	return FPaths::ProjectSavedDir() / TEXT("FASTBuildCache");
}
