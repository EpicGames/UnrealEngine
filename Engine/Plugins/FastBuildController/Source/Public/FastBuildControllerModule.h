// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Containers/Queue.h"
#include "DistributedBuildControllerInterface.h"

class FFastBuildJobProcessor;

class FFastBuildControllerModule : public IDistributedBuildController
{
	bool bSupported;
	bool bModuleInitialized;
	bool bControllerInitialized;

	FThreadSafeCounter NextFileID;
	FThreadSafeCounter NextTaskID;

	FThreadSafeCounter PendingTasksCounter;

	/** The thread spawned for shader compiling. */
	TUniquePtr<FFastBuildJobProcessor> JobDispatcherThread;
	
	// Taken when accessing the PendingTasks and DispatchedTasks members.
	TSharedPtr<FCriticalSection> TasksCS;

	// Queue of tasks submitted by the engine, but not yet dispatched to the controller.
	TQueue<FDistributedBuildTask*> PendingTasks;

	// Map of tasks dispatched to the controller and running within FastBuild, that have not yet finished.
	TMap<uint32, FDistributedBuildTask*> DispatchedTasks;

	const FString RootWorkingDirectory;

	const FString WorkingDirectory;

	FString IntermediateShadersDirectory;

	bool bShutdown;

	void CleanWorkingDirectory();

public:
	FFastBuildControllerModule();
	virtual ~FFastBuildControllerModule();

	virtual void StartupModule() override final;
	virtual void ShutdownModule() override final;

	virtual bool RequiresRelativePaths() override final{ return true; };

	virtual void InitializeController() override final;

	virtual const FString GetName() override final { return FString("FastBuild Controller"); };

	virtual FString RemapPath(const FString& SourcePath) const override final;

	virtual bool IsSupported() override final;

	virtual FString CreateUniqueFilePath() override final;
	virtual TFuture<FDistributedBuildTaskResult> EnqueueTask(const FTaskCommandData& CommandData) override final;

	void EnqueueTask(FDistributedBuildTask* Task);
	FDistributedBuildTask* DequeueTask();

	void RegisterDispatchedTask(FDistributedBuildTask* DispatchedTask);
	void ReEnqueueDispatchedTasks();
	void DeRegisterDispatchedTasks(const TArray<uint32>& InTasksID);
	void ReportJobProcessed(FDistributedBuildTask* CompletedTask, const FTaskResponse& InTaskResponse);
	
	static FFastBuildControllerModule& Get();

	inline bool AreTasksPending() const
	{
		FScopeLock Lock(TasksCS.Get());
		return  !PendingTasks.IsEmpty();
	}

	inline bool AreTasksDispatched() const
	{
		FScopeLock Lock(TasksCS.Get());
		return DispatchedTasks.Num() > 0;
	}

	inline bool AreTasksDispatchedOrPending() const
	{
		FScopeLock Lock(TasksCS.Get());
		return DispatchedTasks.Num() > 0 || !PendingTasks.IsEmpty();
	}

	int32 GetPendingTasksAmount() const { return PendingTasksCounter.GetValue();}

	FString GetRootWorkingDirectory() const { return RootWorkingDirectory;}

	FString GetWorkingDirectory() const { return WorkingDirectory;}

	FString GetIntermediateShadersDirectory() const { return IntermediateShadersDirectory; }

	const TMap<uint32, FDistributedBuildTask*>& GetDispatchedTasks() const
	{
		return DispatchedTasks;
	}

	FCriticalSection* GetTasksCS() const
	{
		return TasksCS.Get();
	}
};
