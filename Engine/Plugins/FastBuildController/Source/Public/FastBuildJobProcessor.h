// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/ThreadSafeBool.h"

class FRunnableThread;

class FFastBuildControllerModule;
class FShaderBatch;
struct FDistributedBuildTask;

enum EFASTBuild_ReturnCodes
{
	FBUILD_OK = 0,
	FBUILD_BUILD_FAILED = -1,
	FBUILD_ERROR_LOADING_BFF = -2,
	FBUILD_BAD_ARGS = -3,
	FBUILD_ALREADY_RUNNING = -4,
	FBUILD_FAILED_TO_SPAWN_WRAPPER = -5,
	FBUILD_FAILED_TO_SPAWN_WRAPPER_FINAL = -6,
	FBUILD_WRAPPER_CRASHED = -7,
};
class FFastBuildJobProcessor : public FRunnable
{
public:

	FFastBuildJobProcessor(FFastBuildControllerModule& InControllerModule);
	virtual ~FFastBuildJobProcessor();

	/** Main loop */
	virtual uint32 Run() override;
	
	/** Aborts the main loop as soon as possible */
	virtual void Stop() override { bForceStop = true; };
	
	/** Creates the threads and starts the main loop */
	void StartThread();

	/** Used to know when this thread has finished the main loop */
	inline bool IsWorkDone() const {return bIsWorkDone;};

protected:

	/** The runnable thread */
	FRunnableThread* Thread;
	
	FProcHandle BuildProcessHandle;
	uint32 BuildProcessID;
	void* PipeRead = nullptr;
	void* PipeWrite = nullptr;

	uint32 LastTimeKickedOffJobs;
	
	FFastBuildControllerModule& ControllerModule;
	
	/** Used to abort the current processing loop */
	FThreadSafeBool bForceStop;
	
	/** Set to true when the main loop finishes*/
	FThreadSafeBool bIsWorkDone;
	
	void WriteScriptFileToDisk(const TArray<FDistributedBuildTask*>& PendingTasks, const FString& ScriptFilename, const FString& WorkerName) const;

	/** Takes all tasks from the queue and creates a FastBuild Script to the kick off the pending jobs */
	void SubmitPendingJobs();
	
	/** Checks the file system to find any results form the submitted jobs */
	void GatherBuildResults() const;

	/** Checks if the FastBuild process is still running properly and handles bad situations */
	void MonitorFastBuildProcess();
};
