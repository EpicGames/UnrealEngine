// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringConv.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFile.h"

struct FDistributedBuildTask;

class FDependencyEnumerator : public IPlatformFile::FDirectoryVisitor
{
public:
	FDependencyEnumerator(FArchive& InScriptFile, const TCHAR* InPrefix, const TCHAR* InExtension)
        : ScriptFile(InScriptFile)
        , Prefix(InPrefix)
        , Extension(InExtension)
		
	{
	}

	virtual bool Visit(const TCHAR* FilenameChar, bool bIsDirectory) override
	{
		if (!bIsDirectory)
		{
			const FString Filename = FString(FilenameChar);

			if ((!Prefix || Filename.Contains(Prefix)) && (!Extension || Filename.EndsWith(Extension)))
			{
				const FString ExtraFile = TEXT("\t\t'") + IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*Filename) + TEXT("'," LINE_TERMINATOR_ANSI);
				ScriptFile.Serialize((void*)StringCast<ANSICHAR>(*ExtraFile, ExtraFile.Len()).Get(), sizeof(ANSICHAR) * ExtraFile.Len());
			}
		}

		return true;
	}

	FArchive& ScriptFile;
	const TCHAR* Prefix;
	const TCHAR* Extension;
};

class FDependencyUniqueArrayEnumerator : public IPlatformFile::FDirectoryVisitor
{
public:
	FDependencyUniqueArrayEnumerator(TArray<FString>& InFullUniqueDependenciesArray, const TCHAR* InPrefix, const TCHAR* InExtension)
        : FullUniqueDependenciesArray(InFullUniqueDependenciesArray)
        , Prefix(InPrefix)
        , Extension(InExtension)
	{
	}

	virtual bool Visit(const TCHAR* FilenameChar, bool bIsDirectory) override
	{
		if (!bIsDirectory)
		{
			const FString Filename = FString(FilenameChar);

			if ((!Prefix || Filename.Contains(Prefix)) && (!Extension || Filename.EndsWith(Extension)))
			{
				FullUniqueDependenciesArray.AddUnique(*IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*Filename));
			}
		}

		return true;
	}

	TArray<FString>& FullUniqueDependenciesArray;
	const TCHAR* Prefix;
	const TCHAR* Extension;
};

class FastBuildUtilities
{
public:

	static void FASTBuildWriteScriptFileHeader(const TArray<FDistributedBuildTask*>& InCompilationTasks, FArchive& ScriptFile, const FString& WorkerName);
	static FArchive* CreateFileHelper(const FString& InFileName);

	static FString GetFastBuildExecutablePath();
	static FString GetFastBuildCache();

protected:
	static void WriteShaderWorkerDependenciesToFile(FArchive& ScriptFile);
	static FString ReplaceEnvironmentVariablesInPath(const FString& ExtraFilePartialPath);
	static void WriteDependenciesToFileUsingWildcardPath(FArchive& ScriptFile, FString ParsedPath);
	static void WritePlatformCompilerDependenciesToFile(FArchive& ScriptFile);
};
